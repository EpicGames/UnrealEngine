{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Animator Kit",
	"Description": "Utilities for Animating in Unreal with Sequencer",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"PluginIconPath": "Resources/Icon128.png",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": true,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "ControlRigSpline",
			"Enabled": true
		},
		{
			"Name": "ControlRigPhysics",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		},
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
		{
			"Name": "GizmoFramework",
			"Enabled": true
		},
		{
			"Name": "PhysicsControl",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "AnimatorKitSettings",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}