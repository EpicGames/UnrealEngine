[/Script/RigVM.RigVMProjectSettings]
+VariantTags=(Name="AnimatorKit_Deformer",Label="Deformer",ToolTip=NSLOCTEXT("RigVMSettings", "AnimatorKit_Deformer_ToolTip", "Deformer rigs can be used to deform characters in Sequencer. These rigs have built-in Deformer Graphs"), Color=(R=0.033105,G=0.266356,B=0.791298,A=1.000000),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)
+VariantTags=(Name="AnimatorKit_Utility",Label="Utility",ToolTip=NSLOCTEXT("RigVMSettings", "AnimatorKit_Utility_ToolTip", "Utility rigs can be used with the Constraint Sytem for on-demand rigging in Sequencer"), Color=(R=0.73791,G=0.099899,B=0.318547,A=1.000000),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)
