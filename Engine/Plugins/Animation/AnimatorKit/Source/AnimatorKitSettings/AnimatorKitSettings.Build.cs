// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnimatorKitSettings : ModuleRules
	{
		public AnimatorKitSettings(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"ControlRigEditor"
				}
			);
		}
	}
}
