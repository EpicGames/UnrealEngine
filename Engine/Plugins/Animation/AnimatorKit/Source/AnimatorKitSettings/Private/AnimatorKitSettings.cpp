// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimatorKitSettings.h"

#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimatorKitSettings)

UAnimatorKitSettings::UAnimatorKitSettings(const FObjectInitializer& ObjectInitializer)
	: UDeveloperSettings(ObjectInitializer)
{}

UAnimatorKitSettings::FOnUpdateSettings UAnimatorKitSettings::OnSettingsChange;

void UAnimatorKitSettings::PostInitProperties()
{
	Super::PostInitProperties(); 

	if (IsTemplate())
	{
		// only import the focus cvar value if it has been set with a higher priority than ECVF_SetByProjectSetting (e.g. ECVF_SetByDeviceProfile) 
		// ImportConsoleVariableValues();
		
		const TCHAR* FocusModeName = TEXT("AnimMode.PendingFocusMode");
		FProperty* FocusModeProp = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimatorKitSettings, bEnableFocusMode));
		IConsoleVariable* FocusModeCVar = IConsoleManager::Get().FindConsoleVariable(FocusModeName);
		if (FocusModeProp && FocusModeCVar)
		{
			if ((FocusModeCVar->GetFlags() & ECVF_SetByMask) < ECVF_SetByProjectSetting)
			{
				FocusModeCVar->Set(static_cast<int32>(bEnableFocusMode), ECVF_SetByProjectSetting);
			}
			else
			{
				FocusModeProp->ImportText_InContainer(*FocusModeCVar->GetString(), this, this, PPF_ConsoleVariable);
			}
		}
	}
}

void UAnimatorKitSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
	
	OnSettingsChange.Broadcast(this);
}
