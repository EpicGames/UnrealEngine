// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimatorKitSettingsModule.h"

#include "AnimatorKitSettings.h"
#include "Modules/ModuleManager.h"

void FAnimatorKitSettingsModule::StartupModule()
{
	UAnimatorKitSettings::OnSettingsChange.AddRaw(this, &FAnimatorKitSettingsModule::UpdateSettings);
	UpdateSettings(GetDefault<UAnimatorKitSettings>());
}

void FAnimatorKitSettingsModule::ShutdownModule()
{
	UAnimatorKitSettings::OnSettingsChange.RemoveAll(this);
}

void FAnimatorKitSettingsModule::UpdateSettings(const UAnimatorKitSettings* InSettings)
{
	// tbd
}


IMPLEMENT_MODULE(FAnimatorKitSettingsModule, AnimatorKitSettings)
