// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"

#include "AnimatorKitSettings.generated.h"

#define UE_API ANIMATORKITSETTINGS_API

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta = (Experimental, DisplayName = "Animator Kit Settings"))
class UAnimatorKitSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(config, EditAnywhere, Category = "Animation Settings|Focus", meta=(ConsoleVariable="AnimMode.PendingFocusMode"))
	bool bEnableFocusMode = true;
    
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateSettings,  const UAnimatorKitSettings*);
	static UE_API FOnUpdateSettings OnSettingsChange;

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};

#undef UE_API
