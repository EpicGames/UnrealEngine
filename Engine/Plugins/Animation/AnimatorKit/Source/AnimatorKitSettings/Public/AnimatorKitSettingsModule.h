// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class UAnimatorKitSettings;

class FAnimatorKitSettingsModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void UpdateSettings(const UAnimatorKitSettings* InSettings);
};
